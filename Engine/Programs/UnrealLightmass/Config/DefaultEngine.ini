[Plugins]
-EnabledPlugins="ExampleDeviceProfileSelector"
+ProgramEnabledPlugins="UdpMessaging"
