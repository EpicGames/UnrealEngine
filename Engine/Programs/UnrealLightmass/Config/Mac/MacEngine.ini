[/Script/MacTargetPlatform.MacTargetSettings]
; Arm64 currently fails to compile due to math intrinsic usage
TargetArchitecture=MacTargetArchitectureIntel
DefaultArchitecture=MacTargetArchitectureIntel
